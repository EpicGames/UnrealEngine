// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// Abstract base class for worker targets.  Not a valid target by itself, hence it is not put into a *.target.cs file.
[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public abstract class DerivedDataBuildWorkerTarget : TargetRules
{
	public DerivedDataBuildWorkerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

		SolutionDirectory = "Programs/BuildWorker";

		bLoggingToMemoryEnabled				= true;
		bUseLoggingInShipping				= true;

		bCompileCEF3						= false;
		bCompileFreeType					= false;
		bCompileICU 						= false;
		bCompileWithAccessibilitySupport	= false;
		bCompileWithPluginSupport			= false;
		bEnableConfigSystem					= false;
		bUsesSlate							= false;
		bUseXGEController					= false;
		bWithLiveCoding						= false;
		bWithServerCode						= false;

		bForceDisableAutomationTests		= true;

		bIsBuildingConsoleApplication		= true;
		bBuildWithEditorOnlyData			= true;
		bBuildDeveloperTools				= false;
		bCompileAgainstEngine				= false;
		bCompileAgainstCoreUObject			= false;
		bCompileAgainstApplicationCore		= false;
		// TODO: I want to use static CRT in the future, but it causes link issues today likely due to 3rd party libraries
		//bUseStaticCRT						= true;

		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bStripUnreferencedSymbols = true;

		// Disable logging to a file to improve worker performance.
		GlobalDefinitions.Add("ALLOW_LOG_FILE=0");
		// Linking against wer.lib/wer.dll causes XGE to bail when the worker is run on a Windows 8 machine.
		// TODO: Is this necessary with a Windows 10 minimum and past the end of support for Windows 10?
		GlobalDefinitions.Add("ALLOW_WINDOWS_ERROR_REPORT_LIB=0");
		// Disable initialization of the crash reporter, which removes another thread being created.
		GlobalDefinitions.Add("NOINITCRASHREPORTER=1");
		// Disable external profiling to improve startup time.
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
	}
}

public class DerivedDataBuildWorker : ModuleRules
{
	public DerivedDataBuildWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PublicIncludePathModuleNames.Add("DerivedDataCache");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
				"Core",
				"DerivedDataCache",
				"Projects",
		});

		if (Target.bCompileAgainstApplicationCore)
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
		}

		// This needs to match the version defined in TargetReceiptBuildWorker.cpp
		AdditionalPropertiesForReceipt.Add("DerivedDataBuildWorkerReceiptVersion", "dab5352e-a5a7-4793-a7a3-1d4acad6aff2");
	}
}
