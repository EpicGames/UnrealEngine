// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compression/CompressedBuffer.h"
#include "Containers/SharedString.h"
#include "Containers/UnrealString.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildAction.h"
#include "DerivedDataBuildDefinition.h"
#include "DerivedDataBuildFunctionRegistry.h"
#include "DerivedDataBuildInputResolver.h"
#include "DerivedDataBuildInputs.h"
#include "DerivedDataBuildOutput.h"
#include "DerivedDataBuildSession.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"
#include "HAL/FileManager.h"
#include "Memory/SharedBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreMisc.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "String/ParseTokens.h"

#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(DerivedDataBuildWorker, "DerivedDataBuildWorker");

DEFINE_LOG_CATEGORY_STATIC(LogDerivedDataBuildWorker, Log, All);

namespace UE::DerivedData
{

class FBuildWorkerProgram : public IBuildInputResolver
{
public:
	bool ParseCommandLine(const TCHAR* CommandLine);
	bool ReportVersions();
	bool Build();

private:
	void BuildComplete(FBuildCompleteParams&& Params) const;

	bool ResolveInputExists(const FBuildAction& Action) const;
	void ResolveInputData(const FBuildAction& Action, IRequestOwner& Owner, FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter) final;

	void GetInputPath(FStringView ActionPath, const FIoHash& RawHash, FStringBuilderBase& OutPath) const;
	void GetOutputPath(FStringView ActionPath, const FIoHash& RawHash, FStringBuilderBase& OutPath) const;

	TUniquePtr<FArchive> OpenInput(FStringView ActionPath, const FIoHash& RawHash) const;
	TUniquePtr<FArchive> OpenOutput(FStringView ActionPath, const FIoHash& RawHash) const;

	FString CommonInputPath;
	FString CommonOutputPath;
	TArray<FString> ActionPaths;
	TArray<FString> VersionPaths;
};

static FSharedBuffer LoadFile(const FString& Path)
{
	FSharedBuffer Buffer;
	if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*Path, FILEREAD_Silent)})
	{
		const int64 TotalSize = Ar->TotalSize();
		FUniqueBuffer MutableBuffer = FUniqueBuffer::Alloc(uint64(TotalSize));
		Ar->Serialize(MutableBuffer.GetData(), TotalSize);
		if (Ar->Close())
		{
			Buffer = MutableBuffer.MoveToShared();
		}
	}
	return Buffer;
}

bool FBuildWorkerProgram::ParseCommandLine(const TCHAR* CommandLine)
{
	TArray<FString> ActionPathPatterns;
	TArray<FString> InputDirectoryPaths;
	TArray<FString> OutputDirectoryPaths;

	for (FString Token; FParse::Token(CommandLine, Token, /*UseEscape*/ false);)
	{
		Token.ReplaceInline(TEXT("\""), TEXT(""));
		const auto GetSwitchValues = [Token = FStringView(Token)](FStringView Match, TArray<FString>& OutValues)
		{
			if (Token.StartsWith(Match))
			{
				OutValues.Emplace(Token.RightChop(Match.Len()));
			}
		};

		GetSwitchValues(TEXT("-B="), ActionPathPatterns);
		GetSwitchValues(TEXT("-Build="), ActionPathPatterns);

		GetSwitchValues(TEXT("-I="), InputDirectoryPaths);
		GetSwitchValues(TEXT("-Input="), InputDirectoryPaths);

		GetSwitchValues(TEXT("-O="), OutputDirectoryPaths);
		GetSwitchValues(TEXT("-Output="), OutputDirectoryPaths);

		GetSwitchValues(TEXT("-V="), VersionPaths);
		GetSwitchValues(TEXT("-Version="), VersionPaths);
	}

	bool bCommandLineIsValid = true;

	if (const int32 InputDirectoryCount = InputDirectoryPaths.Num(); InputDirectoryCount > 1)
	{
		UE_LOGF(LogDerivedDataBuildWorker, Error, "A maximum of one input directory can be specified, but %d were specified.", InputDirectoryCount);
		bCommandLineIsValid = false;
	}
	else if (InputDirectoryCount == 1)
	{
		CommonInputPath = FPaths::ConvertRelativePathToFull(FPaths::LaunchDir(), InputDirectoryPaths[0]);
	}

	if (const int32 OutputDirectoryCount = OutputDirectoryPaths.Num(); OutputDirectoryCount > 1)
	{
		UE_LOGF(LogDerivedDataBuildWorker, Error, "A maximum of one input directory can be specified, but %d were specified.", OutputDirectoryCount);
		bCommandLineIsValid = false;
	}
	else if (OutputDirectoryCount == 1)
	{
		CommonOutputPath = FPaths::ConvertRelativePathToFull(FPaths::LaunchDir(), OutputDirectoryPaths[0]);
	}

	if (ActionPathPatterns.IsEmpty() && VersionPaths.IsEmpty())
	{
		UE_LOGF(LogDerivedDataBuildWorker, Error, "No build action files or version files specified on the command line.");
		bCommandLineIsValid = false;
	}

	for (const FString& ActionPathPattern : ActionPathPatterns)
	{
		FWildcardString ActionPathWildcard(ActionPathPattern);
		if (ActionPathWildcard.ContainsWildcards())
		{
			UE_LOGF(LogDerivedDataBuildWorker, Error, "Wildcards in build action file paths are not supported yet: '%ls'", *ActionPathWildcard);
			bCommandLineIsValid = false;
		}
		else
		{
			ActionPaths.Add(FPaths::ConvertRelativePathToFull(FPaths::LaunchDir(), ActionPathPattern));
		}
	}

	return bCommandLineIsValid;
}

bool FBuildWorkerProgram::ReportVersions()
{
	if (VersionPaths.IsEmpty())
	{
		return true;
	}

	IBuild& BuildSystem = GetBuild();
	FGuid BuildSystemVersion = BuildSystem.GetVersion();
	IBuildFunctionRegistry& BuildFunctionRegistry = BuildSystem.GetFunctionRegistry();
	TMap<FString, FGuid> Functions;
	BuildFunctionRegistry.IterateFunctionVersions([&Functions](FUtf8StringView Function, const FGuid& Version)
	{
		Functions.Emplace(Function, Version);
	});

	FCbWriter Writer;
	Writer.BeginObject();

	Writer << "BuildSystemVersion" << BuildSystemVersion;
	UE_LOGF(LogDerivedDataBuildWorker, Display, "BuildSystemVersion: '%ls'", *WriteToString<64>(BuildSystemVersion));
	UE_LOGF(LogDerivedDataBuildWorker, Display, "Functions:");

	Writer.BeginArray("Functions");
	for (const TPair<FString, FGuid>& Function : Functions)
	{
		Writer.BeginObject();
		Writer << "Name" << Function.Key;
		Writer << "Version" << Function.Value;
		Writer.EndObject();
		UE_LOGF(LogDerivedDataBuildWorker, Display, "%30ls : '%ls'", *Function.Key, *WriteToString<64>(Function.Value));
	}
	Writer.EndArray();

	Writer.EndObject();

	for (const FString& VersionPath : VersionPaths)
	{
		if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileWriter(*FPaths::ConvertRelativePathToFull(FPaths::LaunchDir(), VersionPath))})
		{
			Writer.Save(*Ar);
		}
		else
		{
			UE_LOGF(LogDerivedDataBuildWorker, Error, "Unable to open %ls for writing", *VersionPath);
		}
	}

	return true;
}

bool FBuildWorkerProgram::Build()
{
	if (ActionPaths.IsEmpty())
	{
		return true;
	}

	IBuild& BuildSystem = GetBuild();
	FBuildSession Session = BuildSystem.CreateSession(TEXTVIEW("BuildWorker"), this);
	FRequestOwner Owner(EPriority::Normal);
	{
		FRequestBarrier Barrier(Owner);
		for (const FString& ActionPath : ActionPaths)
		{
			UE_LOGF(LogDerivedDataBuildWorker, Log, "Loading build action '%ls'", *ActionPath);
			FCbObject ActionObject;
			if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*ActionPath, FILEREAD_Silent)})
			{
				*Ar << ActionObject;
			}
			else
			{
				UE_LOGF(LogDerivedDataBuildWorker, Error, "Missing build action '%ls'", *ActionPath);
				return false;
			}
			if (FOptionalBuildAction Action = FBuildAction::Load({ActionPath}, MoveTemp(ActionObject)); Action.IsNull())
			{
				UE_LOGF(LogDerivedDataBuildWorker, Error, "Invalid build action '%ls'", *ActionPath);
				return false;
			}
			else if (!ResolveInputExists(Action.Get()))
			{
				return false;
			}
			else
			{
				Session.Build(Action.Get(), {}, EBuildPolicy::BuildLocal, Owner,
					[this](FBuildCompleteParams&& Params) { BuildComplete(MoveTemp(Params)); });
			}
		}
	}
	Owner.Wait();
	return true;
}

void FBuildWorkerProgram::BuildComplete(FBuildCompleteParams&& Params) const
{
	const FBuildOutput Output = MoveTemp(Params.Output);
	const FSharedString& Name = Output.GetName();
	const FUtf8SharedString& Function = Output.GetFunction();

	for (const FBuildOutputMessage& Message : Output.GetMessages())
	{
		switch (Message.Level)
		{
		case EBuildOutputMessageLevel::Error:
			UE_LOGF(LogDerivedDataBuildWorker, Error, "%ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		case EBuildOutputMessageLevel::Warning:
			UE_LOGF(LogDerivedDataBuildWorker, Warning, "%ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		case EBuildOutputMessageLevel::Display:
			UE_LOGF(LogDerivedDataBuildWorker, Display, "%ls (Build of '%ls' by %ls.)",
				*WriteToString<256>(Message.Message), *Name, *WriteToString<32>(Function));
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	if constexpr (!NO_LOGGING)
	{
		if (GWarn)
		{
			for (const FBuildOutputLog& Log : Output.GetLogs())
			{
				ELogVerbosity::Type Verbosity;
				switch (Log.Level)
				{
				default:
				case EBuildOutputLogLevel::Error:
					Verbosity = ELogVerbosity::Error;
					break;
				case EBuildOutputLogLevel::Warning:
					Verbosity = ELogVerbosity::Warning;
					break;
				}
				GWarn->Log(FName(Log.Category), Verbosity, FString::Printf(TEXT("%s (Build of '%s' by %s.)"),
					*WriteToString<256>(Log.Message), *Name, *WriteToString<32>(Function)));
			}
		}
	}

	for (const FValueWithId& Value : Output.GetValues())
	{
		if (Value.HasData())
		{
			if (TUniquePtr<FArchive> Ar = OpenOutput(Name, Value.GetRawHash()))
			{
				Value.GetData().Save(*Ar);
				if (Ar->Close())
				{
					continue;
				}
			}
			UE_LOGF(LogDerivedDataBuildWorker, Error,
				"Failed to store build output %ls for build of '%ls' by %ls.",
				*WriteToString<48>(Value.GetRawHash()), *Name, *WriteToString<32>(Function));
		}
	}

	const FString OutputPath = FPathViews::ChangeExtension(Name, TEXT("output"));
	if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileWriter(*OutputPath, FILEWRITE_Silent)})
	{
		FCbWriter OutputWriter;
		Output.Save(OutputWriter);
		OutputWriter.Save(*Ar);
	}
	else
	{
		UE_LOGF(LogDerivedDataBuildWorker, Error,
			"Failed to store build output to '%ls' for build of '%ls' by %ls.",
			*OutputPath, *Name, *WriteToString<32>(Function));
	}
}

bool FBuildWorkerProgram::ResolveInputExists(const FBuildAction& Action) const
{
	bool bValid = true;
	Action.IterateInputs([this, &Action, &bValid](FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
	{
		TStringBuilder<256> Path;
		GetInputPath(Action.GetName(), RawHash, Path);
		if (!IFileManager::Get().FileExists(*Path))
		{
			bValid = false;
			UE_LOGF(LogDerivedDataBuildWorker, Error,
				"Input '%ls' with raw hash %ls is missing for build of '%ls' by %ls.", *WriteToString<64>(Key),
				*WriteToString<48>(RawHash), *Action.GetName(), *WriteToString<32>(Action.GetFunction()));
		}
	});
	return bValid;
}

void FBuildWorkerProgram::ResolveInputData(const FBuildAction& Action, IRequestOwner& Owner, FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter)
{
	EStatus Status = EStatus::Ok;
	TArray<FBuildInputDataByKey> Inputs;
	Action.IterateInputs([this, &Action, &Filter, &Inputs, &Status](FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
	{
		if (Filter && !Filter(Key))
		{
			return;
		}
		if (TUniquePtr<FArchive> Ar = OpenInput(Action.GetName(), RawHash))
		{
			Inputs.Add({Key, FCompressedBuffer::Load(*Ar)});
		}
		else
		{
			Status = EStatus::Error;
			UE_LOGF(LogDerivedDataBuildWorker, Error,
				"Input '%ls' with raw hash %ls is missing for build of '%ls' by %ls.", *WriteToString<64>(Key),
				*WriteToString<48>(RawHash), *Action.GetName(), *WriteToString<32>(Action.GetFunction()));
		}
	});
	OnResolved({Inputs, Status});
}

void FBuildWorkerProgram::GetInputPath(FStringView ActionPath, const FIoHash& RawHash, FStringBuilderBase& OutPath) const
{
	if (CommonInputPath.IsEmpty())
	{
		FPathViews::Append(OutPath, FPathViews::GetPath(ActionPath), TEXT("Inputs"), RawHash);
	}
	else
	{
		FPathViews::Append(OutPath, CommonInputPath, RawHash);
	}
}

void FBuildWorkerProgram::GetOutputPath(FStringView ActionPath, const FIoHash& RawHash, FStringBuilderBase& OutPath) const
{
	if (CommonOutputPath.IsEmpty())
	{
		FPathViews::Append(OutPath, FPathViews::GetPath(ActionPath), TEXT("Outputs"), RawHash);
	}
	else
	{
		FPathViews::Append(OutPath, CommonOutputPath, RawHash);
	}
}

TUniquePtr<FArchive> FBuildWorkerProgram::OpenInput(FStringView ActionPath, const FIoHash& RawHash) const
{
	TStringBuilder<256> Path;
	GetInputPath(ActionPath, RawHash, Path);
	return TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Path, FILEREAD_Silent));
}

TUniquePtr<FArchive> FBuildWorkerProgram::OpenOutput(FStringView ActionPath, const FIoHash& RawHash) const
{
	TStringBuilder<256> Path;
	GetOutputPath(ActionPath, RawHash, Path);
	return TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Path, FILEWRITE_NoReplaceExisting));
}

} // UE::DerivedData

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

DERIVEDDATABUILDWORKER_API int32 BuildWorkerMain(int32 ArgC, TCHAR* ArgV[], const TCHAR* ExtraArgs, const TCHAR* LoadModules)
{
	ON_SCOPE_EXIT
	{
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	const FTaskTagScope Scope(ETaskTag::EGameThread);

	uint64 WorkerStartTime = FPlatformTime::Cycles64();

	if (const int32 ErrorLevel = GEngineLoop.PreInit(ArgC, ArgV, ExtraArgs))
	{
		return ErrorLevel;
	}

	UE::DerivedData::FBuildWorkerProgram Program;

	if (!Program.ParseCommandLine(FCommandLine::Get()))
	{
		return 1;
	}

	FModuleManager& ModuleManager = FModuleManager::Get();

	// Load DerivedDataCache before the rest of the modules because that will make it the last to
	// shut down on exit. This is an approximation of the module load/unload order in the editor.
	ModuleManager.LoadModule(TEXT("DerivedDataCache"));

	TArray<FName> Modules;
	UE::String::ParseTokens(LoadModules, '+', [&ModuleManager](FStringView ModulePattern)
	{
		TArray<FName> Modules;
		ModuleManager.FindModules(*WriteToString<64>(ModulePattern), Modules);
		for (FName Module : Modules)
		{
			ModuleManager.LoadModule(Module);
		}
	});

	if (!Program.ReportVersions())
	{
		return 1;
	}

	if (!Program.Build())
	{
		return 1;
	}

	UE_LOGF(LogDerivedDataBuildWorker, Log, "Worker completed in %.3fms", FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64()-WorkerStartTime));

	return 0;
}

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	return BuildWorkerMain(ArgC, ArgV, TEXT("-Unattended"), TEXT("*"));
}
